{
	"FileVersion": 3,
	"Version": 1,
	"VersionName": "1.0",
	"FriendlyName": "Gameplay Tags Toolset",
	"Description": "Toolset for reading and managing gameplay tags via the AI Toolset Registry.",
	"Category": "Other",
	"EditorOnly": true,
	"IsExperimentalVersion": true,
	"NoRedist": true,
	"EnabledByDefault": false,
	"Modules": [
		{
			"Name": "GameplayTagsToolset",
			"Type": "Editor",
			"LoadingPhase": "PostEngineInit",
			"TargetAllowList": ["Editor"]
		}
	],
	"Plugins": [
		{
			"Name": "ToolsetRegistry",
			"Enabled": true
		},
		{
			"Name": "GameplayTagsEditor",
			"Enabled": true
		}
	]
}
