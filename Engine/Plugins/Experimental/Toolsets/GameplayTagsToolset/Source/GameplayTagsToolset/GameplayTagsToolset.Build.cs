// Copyright Epic Games, Inc. All Rights Reserved.

using UnrealBuildTool;

public class GameplayTagsToolset : ModuleRules
{
	public GameplayTagsToolset(ReadOnlyTargetRules Target) : base(Target)
	{
		PCHUsage = PCHUsageMode.UseExplicitOrSharedPCHs;
		IWYUSupport = IWYUSupport.Full;
		bUseUnity = true;

		PublicDependencyModuleNames.Add("Core");

		PrivateDependencyModuleNames.AddRange(new string[]
		{
			"AssetRegistry",
			"CoreUObject",
			"EditorScriptingUtilities",
			"Engine",
			"GameplayTags",
			"GameplayTagsEditor",
			"Kismet",
			"ToolsetRegistry",
			"UnrealEd",
		});
	}
}
