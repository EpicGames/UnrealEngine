// Copyright Epic Games, Inc. All Rights Reserved.

#include "GameplayTagsToolset/GameplayTagsToolset.h"

#include "GameplayTagsEditorModule.h"
#include "GameplayTagsManager.h"
#include "Kismet/KismetSystemLibrary.h"
#include "AssetRegistry/IAssetRegistry.h"
#include "Modules/ModuleManager.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(GameplayTagsToolset)

namespace
{
	TArray<FString> CollectDescendantTagNames(const TSharedPtr<FGameplayTagNode>& ParentNode)
	{
		TArray<FString> TagNames;
		TArray<TSharedPtr<FGameplayTagNode>> NodesToTraverse = ParentNode->GetChildTagNodes();
		while (NodesToTraverse.Num() > 0)
		{
			TSharedPtr<FGameplayTagNode> Node = NodesToTraverse.Pop(EAllowShrinking::No);
			if (Node.IsValid())
			{
				TagNames.Add(Node->GetCompleteTagName().ToString());
				NodesToTraverse.Append(Node->GetChildTagNodes());
			}
		}
		return TagNames;
	}

	IGameplayTagsEditorModule& GetModule()
	{
		return FModuleManager::LoadModuleChecked<IGameplayTagsEditorModule>(
			TEXT("GameplayTagsEditor"));
	}
}

TArray<FString> UGameplayTagsToolset::ListTags(const FString& ParentTag)
{
	UGameplayTagsManager& TagManager = UGameplayTagsManager::Get();
	TArray<FString> TagNames;

	if (ParentTag.IsEmpty())
	{
		FGameplayTagContainer AllTags;
		TagManager.RequestAllGameplayTags(AllTags, false);
		for (const FGameplayTag& Tag : AllTags)
		{
			TagNames.Add(Tag.GetTagName().ToString());
		}
	}
	else
	{
		TSharedPtr<FGameplayTagNode> ParentNode = TagManager.FindTagNode(FName(*ParentTag));
		if (ParentNode.IsValid())
		{
			TagNames = CollectDescendantTagNames(ParentNode);
		}
	}

	TagNames.Sort();
	return TagNames;
}

FGameplayTagInfo UGameplayTagsToolset::GetTagInfo(const FString& TagName)
{
	TSharedPtr<FGameplayTagNode> TagNode =
		UGameplayTagsManager::Get().FindTagNode(FName(*TagName));

	if (!TagNode.IsValid())
	{
		UKismetSystemLibrary::RaiseScriptError(
			FString::Printf(TEXT("Gameplay tag '%s' does not exist."), *TagName));
		return FGameplayTagInfo();
	}

	FGameplayTagInfo Info;
#if WITH_EDITORONLY_DATA
	Info.Comment = TagNode->GetDevComment();
	Info.Source = TagNode->GetFirstSourceName().ToString();
#endif

	for (const TSharedPtr<FGameplayTagNode>& Child : TagNode->GetChildTagNodes())
	{
		if (Child.IsValid())
		{
			Info.Children.Add(Child->GetCompleteTagName().ToString());
		}
	}
	return Info;
}

void UGameplayTagsToolset::AddTag(
	const FString& TagName, const FString& Comment, const FString& TagSource)
{
	if (TagName.IsEmpty())
	{
		UKismetSystemLibrary::RaiseScriptError(TEXT("Tag name cannot be empty."));
		return;
	}

	const FName SourceName = TagSource.IsEmpty() ? NAME_None : FName(*TagSource);
	if (!GetModule().AddNewGameplayTagToINI(TagName, Comment, SourceName))
	{
		UKismetSystemLibrary::RaiseScriptError(
			FString::Printf(TEXT("Failed to add gameplay tag '%s'."), *TagName));
	}
}

void UGameplayTagsToolset::RemoveTag(const FString& TagName)
{
	TSharedPtr<FGameplayTagNode> TagNode =
		UGameplayTagsManager::Get().FindTagNode(FName(*TagName));
	if (!TagNode.IsValid())
	{
		UKismetSystemLibrary::RaiseScriptError(
			FString::Printf(TEXT("Gameplay tag '%s' does not exist."), *TagName));
		return;
	}
	if (!GetModule().DeleteTagFromINI(TagNode))
	{
		UKismetSystemLibrary::RaiseScriptError(
			FString::Printf(TEXT("Failed to remove gameplay tag '%s'."), *TagName));
	}
}

void UGameplayTagsToolset::RenameTag(const FString& OldTagName, const FString& NewTagName)
{
	if (OldTagName.IsEmpty() || NewTagName.IsEmpty())
	{
		UKismetSystemLibrary::RaiseScriptError(TEXT("Tag names cannot be empty."));
		return;
	}

	if (!UGameplayTagsManager::Get().FindTagNode(FName(*OldTagName)).IsValid())
	{
		UKismetSystemLibrary::RaiseScriptError(
			FString::Printf(TEXT("Gameplay tag '%s' does not exist."), *OldTagName));
		return;
	}

	if (!GetModule().RenameTagInINI(OldTagName, NewTagName))
	{
		UKismetSystemLibrary::RaiseScriptError(FString::Printf(
			TEXT("Failed to rename gameplay tag '%s' to '%s'."), *OldTagName, *NewTagName));
	}
}

TArray<FString> UGameplayTagsToolset::FindReferencersByTag(const FString& TagName)
{
	if (TagName.IsEmpty())
	{
		UKismetSystemLibrary::RaiseScriptError(TEXT("Tag name cannot be empty."));
		return TArray<FString>();
	}

	const FGameplayTag Tag = UGameplayTagsManager::Get().RequestGameplayTag(FName(*TagName), false);
	if (!Tag.IsValid())
	{
		UKismetSystemLibrary::RaiseScriptError(
			FString::Printf(TEXT("Gameplay tag '%s' does not exist."), *TagName));
		return TArray<FString>();
	}

	const FAssetIdentifier TagIdentifier(FGameplayTag::StaticStruct(), Tag.GetTagName());

	TArray<FAssetIdentifier> Referencers;
	IAssetRegistry::Get()->GetReferencers(TagIdentifier, Referencers,
		UE::AssetRegistry::EDependencyCategory::SearchableName);

	TArray<FString> Result;
	Result.Reserve(Referencers.Num());
	for (const FAssetIdentifier& Id : Referencers)
	{
		Result.Add(Id.PackageName.ToString());
	}
	return Result;
}
