// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"

#include "ToolsetRegistry/ToolsetDefinition.h"

#include "GameplayTagsToolset.generated.h"

/// Describes the properties of a gameplay tag.
USTRUCT(BlueprintType)
struct FGameplayTagInfo
{
	GENERATED_BODY()

	/// The developer comment describing the tag's purpose.
	UPROPERTY(BlueprintReadWrite, Category = "GameplayTag")
	FString Comment;

	/// The name of the INI source file that defines this tag (e.g. "DefaultGameplayTags.ini").
	UPROPERTY(BlueprintReadWrite, Category = "GameplayTag")
	FString Source;

	/// The fully-qualified names of this tag's immediate children.
	UPROPERTY(BlueprintReadWrite, Category = "GameplayTag")
	TArray<FString> Children;
};

/// Provides tools for reading and managing gameplay tags in the project's GameplayTagsManager.
UCLASS(BlueprintType, Hidden)
class UGameplayTagsToolset : public UToolsetDefinition
{
	GENERATED_BODY()

public:
	/**
	 * Returns gameplay tags registered in the project.
	 * @param ParentTag If non-empty, only tags that are descendants of this tag are returned.
	 *   For example, passing "Character.State" returns "Character.State.Dead",
	 *   "Character.State.Stunned", etc. Pass an empty string to return all tags.
	 * @return A sorted list of fully-qualified tag names.
	 */
	UFUNCTION(meta = (AICallable), Category = "GameplayTags")
	static TArray<FString> ListTags(const FString& ParentTag);

	/**
	 * Returns detailed information about a specific gameplay tag.
	 * @param TagName The fully-qualified name of the tag, e.g. "Character.State.Dead".
	 * @return The tag's developer comment, source INI file, and immediate children.
	 *   Raises a script error if the tag does not exist.
	 */
	UFUNCTION(meta = (AICallable), Category = "GameplayTags")
	static FGameplayTagInfo GetTagInfo(const FString& TagName);

	/**
	 * Adds a new gameplay tag to the project.
	 * This should ONLY be called after getting explicit direction or permission from the user.
	 * @param TagName The fully-qualified name of the tag to add, e.g. "Character.State.Dead".
	 * @param Comment An optional developer comment describing the tag's purpose.
	 * @param TagSource The INI source to add the tag to. Uses the default source if empty.
	 */
	UFUNCTION(meta = (AICallable), Category = "GameplayTags")
	static void AddTag(const FString& TagName, const FString& Comment, const FString& TagSource);

	/**
	 * Removes a gameplay tag from the project.
	 * This should ONLY be called after getting explicit direction or permission from the user.
	 * @param TagName The fully-qualified name of the tag to remove, e.g. "Character.State.Dead".
	 */
	UFUNCTION(meta = (AICallable), Category = "GameplayTags")
	static void RemoveTag(const FString& TagName);

	/**
	 * Renames a gameplay tag, updating all references in the project.
	 * This should ONLY be called after getting explicit direction or permission from the user.
	 * @param OldTagName The fully-qualified name of the tag to rename.
	 * @param NewTagName The new fully-qualified name for the tag.
	 */
	UFUNCTION(meta = (AICallable), Category = "GameplayTags")
	static void RenameTag(const FString& OldTagName, const FString& NewTagName);

	/**
	 * Returns assets that reference a gameplay tag.
	 * @param TagName The fully-qualified gameplay tag to search for, e.g. "Character.State.Dead".
	 * @return A list of package paths that reference the tag.
	 */
	UFUNCTION(meta = (AICallable), Category = "GameplayTags")
	static TArray<FString> FindReferencersByTag(const FString& TagName);
};
