// Copyright Epic Games, Inc. All Rights Reserved.

#include "GameplayTagsToolset.h"

#include "Editor.h"
#include "Misc/CoreDelegates.h"
#include "Modules/ModuleInterface.h"
#include "Modules/ModuleManager.h"
#include "ToolsetRegistry/UToolsetRegistry.h"

class FGameplayTagsToolsetModule : public IModuleInterface
{
public:
	virtual void StartupModule() override
	{
		UToolsetRegistry::RegisterToolsetClass(UGameplayTagsToolset::StaticClass());
	}

	virtual void ShutdownModule() override
	{
		UToolsetRegistry::UnregisterToolsetClass(UGameplayTagsToolset::StaticClass());
	}
};

IMPLEMENT_MODULE(FGameplayTagsToolsetModule, GameplayTagsToolset);
