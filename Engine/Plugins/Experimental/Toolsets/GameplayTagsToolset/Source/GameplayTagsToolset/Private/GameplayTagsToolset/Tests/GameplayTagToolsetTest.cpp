// Copyright Epic Games, Inc. All Rights Reserved.

#include "GameplayTagsToolset/Tests/GameplayTagToolsetTest.h"

#include "AssetRegistry/AssetRegistryModule.h"
#include "AssetToolsModule.h"
#include "Containers/Array.h"
#include "CoreMinimal.h"
#include "EditorAssetLibrary.h"
#include "GameplayTagsManager.h"
#include "HAL/FileManager.h"
#include "Misc/AutomationTest.h"
#include "Misc/FileHelper.h"
#include "Misc/Guid.h"
#include "Misc/PackageName.h"
#include "Misc/Paths.h"
#include "Templates/UniquePtr.h"
#include "ToolsetRegistry/ToolCallExceptionHandler.h"

#include "GameplayTagsToolset/GameplayTagsToolset.h"

#if WITH_DEV_AUTOMATION_TESTS

namespace
{
	const auto TestFlags =
		EAutomationTestFlags::EditorContext |
		EAutomationTestFlags::ProductFilter |
		EAutomationTestFlags::CriticalPriority;
}

BEGIN_DEFINE_SPEC(FGameplayTagsToolsetSpec, "AI.Toolsets.GameplayTagsToolset.GameplayTagsToolsetSpec",
	TestFlags)

	// Ensure no blueprint script exceptions were caught and clear the handler.
	void ExpectNoException()
	{
		check(ExceptionHandler.IsValid());
		TestEqual(TEXT("Error"), *ExceptionHandler->GetException(), TEXT(""));
		ExceptionHandler.Reset();
	}

	// Ensure blueprint script exception was caught and contains the error message, then clear the
	// handler.
	void ExpectExceptionContains(const FString& ErrorMessage)
	{
		check(ExceptionHandler.IsValid());
		TestTrue(
			*FString::Printf(TEXT("Error contains '%s'"), *ErrorMessage),
			ExceptionHandler->GetException().Contains(ErrorMessage));
		ExceptionHandler.Reset();
	}

	// Ensure blueprint script exception was caught and clear the handler.
	void ExpectException()
	{
		check(ExceptionHandler.IsValid());
		TestNotEqual(TEXT("Has Error"), *ExceptionHandler->GetException(), TEXT(""));
		ExceptionHandler.Reset();
	}

	// Add a test tag that is cleaned up after the test case.
	// Returns the name of the new tag.
	FString AddTestTag(
		const FString& TagNamePostfix,
		const FString& Comment = DefaultTagComment)
	{
		check(ExceptionHandler);
		FString TagName = TestTagName + TagNamePostfix;
		ExceptionHandler->CaptureErrorsIn(
			[this, &TagName, &Comment]()
			{
				UGameplayTagsToolset::AddTag(TagName, Comment, TestTagSourceName.ToString());
			});
		AddedTagNames.Add(TagName);
		return TagName;
	}

private:
	FString TestTagName;
	FString RenamedTagName;
	TArray<FString> AddedTagNames;
	TUniquePtr<UE::ToolsetRegistry::FToolCallExceptionHandler> ExceptionHandler;

private:
	static const FString NonExistentTag;
	static const FString DefaultTagComment;
	// Tag source backed by the project saved directory so writes succeed on build machines,
	// where the project config directory is read-only. AddTagIniSearchPath registers it with
	// the full writable path so AddNewGameplayTagToINI can find and use it.
	static const FName TestTagSourceName;
END_DEFINE_SPEC(FGameplayTagsToolsetSpec)

const FString FGameplayTagsToolsetSpec::NonExistentTag = TEXT("ZZZ.NonExistent.XYZABC123456");
const FString FGameplayTagsToolsetSpec::DefaultTagComment = TEXT("Automated test tag");
const FName FGameplayTagsToolsetSpec::TestTagSourceName = FName(TEXT("GameplayTagsToolsetTest.ini"));

void FGameplayTagsToolsetSpec::Define()
{
	using namespace UE::ToolsetRegistry;

	BeforeEach([this]()
	{
		// Register a tag INI source in the project saved directory so that
		// AddNewGameplayTagToINI writes to a location that is writable on all machines,
		// including build machines where the project config directory is read-only.
		//
		// AddTagIniSearchPath discovers .ini files in the directory and records their full
		// paths. EditorRefreshGameplayTagTree then registers each file as a tag source with
		// its correct ConfigFileName (pointing into the saved directory), so that subsequent
		// AddTag calls write to the right writable location.
		const FString TestTagIniDir =
			FPaths::ProjectSavedDir() / TEXT("AutomationTests") / TEXT("GameplayTags");
		const FString TestTagIniPath = TestTagIniDir / TestTagSourceName.ToString();

		IFileManager::Get().MakeDirectory(*TestTagIniDir, /*Tree=*/true);
		if (!FPaths::FileExists(TestTagIniPath))
		{
			FFileHelper::SaveStringToFile(TEXT(""), *TestTagIniPath);
		}

		UGameplayTagsManager& Manager = UGameplayTagsManager::Get();
		Manager.AddTagIniSearchPath(TestTagIniDir);
		Manager.EditorRefreshGameplayTagTree();

		ExceptionHandler = MakeUnique<UE::ToolsetRegistry::FToolCallExceptionHandler>();
		TestTagName = FString::Printf(
			TEXT("ToolsetTest.AutoTest%s"),
			*FGuid::NewGuid().ToString(EGuidFormats::Digits).Left(8));
		RenamedTagName = TestTagName + TEXT("Renamed");
		AddedTagNames.Add(RenamedTagName);  // Clean up the renamed tag if it exists.
	});

	AfterEach([this]()
	{
		if (ExceptionHandler.IsValid()) ExpectNoException();
		// Clean up test tags.
		UGameplayTagsManager& Manager = UGameplayTagsManager::Get();
		for (const FString& TagName : AddedTagNames)
		{
			if (Manager.FindTagNode(FName(*TagName)).IsValid())
			{
				UGameplayTagsToolset::RemoveTag(TagName);
			}
		}
		TestTagName.Empty();
		RenamedTagName.Empty();
		AddedTagNames.Empty();
	});

	Describe(TEXT("ListTags"), [this]()
	{
		It(TEXT("Returns a sorted list"), [this]()
		{
			AddTestTag(TEXT("foo"));
			AddTestTag(TEXT("foo.bar"));
			ExceptionHandler->CaptureErrorsIn([this]()
				{
					TArray<FString> Tags = UGameplayTagsToolset::ListTags(FString());
					TArray<FString> Sorted = Tags;
					Sorted.Sort();
					TestEqual("Sorted", Tags, Sorted);
				});
		});

		It(TEXT("Returns empty array for a non-existent parent tag"), [this]()
		{
			ExceptionHandler->CaptureErrorsIn([this]()
				{
					TArray<FString> Tags = UGameplayTagsToolset::ListTags(NonExistentTag);
					TestEqual("Empty", Tags.Num(), 0);
				});
		});
	});

	Describe(TEXT("GetTagInfo"), [this]()
	{
		It(TEXT("Fails for a non-existent tag"), [this]()
		{
			ExceptionHandler->CaptureErrorsIn([this]()
				{
					FGameplayTagInfo Info = UGameplayTagsToolset::GetTagInfo(NonExistentTag);
#if WITH_EDITORONLY_DATA
					TestTrue("Comment empty", Info.Comment.IsEmpty());
					TestTrue("Source empty", Info.Source.IsEmpty());
#endif  // WITH_EDITORONLY_DATA
					TestEqual("No children", Info.Children.Num(), 0);
				});
			ExpectException();
		});

		It(TEXT("Returns info for an existing tag"), [this]()
		{
			FString TagName = AddTestTag(TEXT(".foo"));
			TArray<FString> Children =
			{
				AddTestTag(TEXT(".foo.bish")),
				AddTestTag(TEXT(".foo.bosh")),
			};
			ExceptionHandler->CaptureErrorsIn([this, &TagName, &Children]()
				{
					FGameplayTagInfo Info = UGameplayTagsToolset::GetTagInfo(TagName);
#if WITH_EDITORONLY_DATA
					TestEqual(TEXT("Comment"), Info.Comment, DefaultTagComment);
					TestEqual(TEXT("Source"), Info.Source, TestTagSourceName.ToString());
#endif  // WITH_EDITORONLY_DATA
					if (TestEqual(TEXT("NumChildren"), Info.Children.Num(), Children.Num()))
					{
						for (int i = 0; i < Children.Num(); ++i)
						{
							TestEqual(
								*FString::Printf(TEXT("Children[%d]"), i),
								Info.Children[i], Children[i]);
						}
					}
				});
			ExpectNoException();
		});
	});

	Describe(TEXT("AddTag and RemoveTag"), [this]()
	{
		It(TEXT("Can add a new tag"), [this]()
		{
			FString TagName;
			ExceptionHandler->CaptureErrorsIn(
				[this, &TagName]()
				{
					TagName = AddTestTag(TEXT("test"));
				});
			ExpectNoException();
			TestTrue(TEXT("Exists in manager"),
				UGameplayTagsManager::Get().FindTagNode(FName(*TagName)).IsValid());
		});

		It(TEXT("Fails to add empty tag"), [this]()
		{
			ExceptionHandler->CaptureErrorsIn(
				[this]()
				{
					UGameplayTagsToolset::AddTag(FString(), FString(), FString());
				});
			ExpectException();
		});

		It(TEXT("Can remove an existing tag"), [this]()
		{
			FString TagName = AddTestTag(TEXT(""));
			ExceptionHandler->CaptureErrorsIn(
				[this, &TagName]()
				{
					UGameplayTagsToolset::RemoveTag(TagName);
				});
			ExpectNoException();
			TestFalse(TEXT("No longer in manager"),
				UGameplayTagsManager::Get().FindTagNode(FName(*TagName)).IsValid());
		});

		It(TEXT("Fails to remove a non-existent tag"), [this]()
		{
			ExceptionHandler->CaptureErrorsIn(
				[this]()
				{
					UGameplayTagsToolset::RemoveTag(NonExistentTag);
				});
			ExpectExceptionContains(NonExistentTag);
		});
	});

	Describe(TEXT("RenameTag"), [this]()
	{
		It(TEXT("Can rename an existing tag"), [this]()
		{
			FString TagName = AddTestTag(TEXT(""));
			ExceptionHandler->CaptureErrorsIn(
				[this, &TagName]()
				{
					UGameplayTagsToolset::RenameTag(TagName, RenamedTagName);
				});
			ExpectNoException();
			TestTrue(TEXT("New name exists"),
				UGameplayTagsManager::Get().FindTagNode(FName(*RenamedTagName)).IsValid());
		});

		It(TEXT("Fails when source tag name does not exist"), [this]()
		{
			ExceptionHandler->CaptureErrorsIn(
				[this]()
				{
					UGameplayTagsToolset::RenameTag(NonExistentTag, RenamedTagName);
				});
			ExpectExceptionContains(NonExistentTag);
		});

		It(TEXT("Fails with an empty source tag name"), [this]()
		{
			ExceptionHandler->CaptureErrorsIn(
				[this]()
				{
					UGameplayTagsToolset::RenameTag(FString(), RenamedTagName);
				});
			ExpectException();
		});

		It(TEXT("Fails with an empty target tag name"), [this]()
		{
			ExceptionHandler->CaptureErrorsIn(
				[this]()
				{
					UGameplayTagsToolset::RenameTag(TestTagName, FString());
				});
			ExpectException();
		});
	});

	Describe(TEXT("FindReferencersByTag"), [this]()
	{
		It(TEXT("Returns empty array for a tag with no referencers"), [this]()
		{
			FString TagName = AddTestTag(TEXT(".norefs"));
			ExceptionHandler->CaptureErrorsIn([this, &TagName]()
				{
					TArray<FString> Referencers =
						UGameplayTagsToolset::FindReferencersByTag(TagName);
					TestEqual("Empty", Referencers.Num(), 0);
				});
			ExpectNoException();
		});

		It(TEXT("Fails for a non-existent tag"), [this]()
		{
			ExceptionHandler->CaptureErrorsIn([this]()
				{
					TArray<FString> Referencers =
						UGameplayTagsToolset::FindReferencersByTag(NonExistentTag);
					TestEqual("Empty", Referencers.Num(), 0);
				});
			ExpectExceptionContains(NonExistentTag);
		});

		It(TEXT("Fails with an empty tag name"), [this]()
		{
			ExceptionHandler->CaptureErrorsIn([this]()
				{
					UGameplayTagsToolset::FindReferencersByTag(FString());
				});
			ExpectException();
		});

		It(TEXT("Returns referencers for a tag used by a saved asset"), [this]()
		{
			FString TagName = AddTestTag(TEXT(".hasrefs"));

			// Mount a transient content path for test assets.
			const FString MountPoint = TEXT("/Automation/");
			FPackageName::RegisterMountPoint(
				*MountPoint, FPaths::AutomationTransientDir());

			// Create a minimal asset that contains our test tag via
			// the editor asset pipeline (avoids data validation issues
			// with raw UPackage::SavePackage).
			const FString AssetName = TEXT("TagRefTestAsset");
			const FString PackagePath = MountPoint + AssetName;
			IAssetTools& AssetTools =
				FAssetToolsModule::GetModule().Get();
			UObject* CreatedAsset = AssetTools.CreateAsset(
				AssetName, MountPoint,
				UGameplayTagTestAsset::StaticClass(), nullptr);
			if (!TestNotNull(TEXT("Asset created"), CreatedAsset))
			{
				return;
			}

			UGameplayTagTestAsset* TestAsset =
				CastChecked<UGameplayTagTestAsset>(CreatedAsset);
			TestAsset->Tags.AddTag(
				UGameplayTagsManager::Get().RequestGameplayTag(
					FName(*TagName)));

			// Save through the editor library so the asset registry
			// picks up the searchable name.
			UEditorAssetLibrary::SaveAsset(PackagePath);

			// Ensure the asset registry has processed the saved file's
			// searchable name dependencies before we query.
			const FString FilePath =
				FPaths::AutomationTransientDir() / TEXT("TagRefTestAsset.uasset");
			IAssetRegistry& AssetRegistry =
				FModuleManager::LoadModuleChecked<FAssetRegistryModule>(
					TEXT("AssetRegistry")).Get();
			AssetRegistry.ScanFilesSynchronous({FilePath}, true /* bForceRescan */);

			// Query and verify.
			ExceptionHandler->CaptureErrorsIn([this, &TagName, &PackagePath]()
				{
					TArray<FString> Referencers =
						UGameplayTagsToolset::FindReferencersByTag(TagName);
					TestTrue("Has referencers", Referencers.Num() > 0);
					TestTrue("Contains test asset",
						Referencers.Contains(PackagePath));
				});
			ExpectNoException();

			// Cleanup.
			UEditorAssetLibrary::DeleteAsset(PackagePath);
			FPackageName::UnRegisterMountPoint(*MountPoint,
				FPaths::AutomationTransientDir());
		});
	});
}

#endif  // WITH_DEV_AUTOMATION_TESTS
