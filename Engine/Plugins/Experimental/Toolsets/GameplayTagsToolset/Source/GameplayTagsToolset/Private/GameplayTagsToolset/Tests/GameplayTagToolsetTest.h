// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"

#include "Engine/DataAsset.h"
#include "GameplayTagContainer.h"
#include "GameplayTagToolsetTest.generated.h"

// Minimal test asset that holds a gameplay tag container for searchable name tests.
UCLASS(Hidden)
class UGameplayTagTestAsset : public UPrimaryDataAsset
{
	GENERATED_BODY()

public:
	UPROPERTY()
	FGameplayTagContainer Tags;
};
